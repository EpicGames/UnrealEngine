{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Datasmith MVR",
	"Description": "Enabled support to import .udatasmith files with MVR content.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DatasmithMVRTranslator",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DMXEngine",
			"Enabled": true
		},
		{
			"Name": "DatasmithContent",
			"Enabled": true
		},
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		},
	]
}
