// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DatasmithMVRTranslator : ModuleRules
{
	public DatasmithMVRTranslator(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
		});


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"DatasmithContent",
				"DatasmithCore",
				"DatasmithNativeTranslator",
				"DatasmithTranslator",
				"DMXEditor",
				"DMXRuntime",
				"Engine",
				"Slate",
			}
		);
	}
}
