// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"


#include "DatasmithMVRImportOptions.generated.h"


UCLASS(Config = EditorPerProjectUserSettings)
class UDatasmithMVRImportOptions 
	: public UDatasmithOptionsBase
{
	GENERATED_BODY()

public:
	/** If set to true, datasmith elements that correspond to an entry in the MVR are replaced with GDTF Actors */
	UPROPERTY(Config, EditAnywhere, Category = MVR)
	bool bImportMVR = true;
};
