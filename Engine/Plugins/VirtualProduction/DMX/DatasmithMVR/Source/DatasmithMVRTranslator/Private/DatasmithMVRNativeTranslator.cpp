// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMVRNativeTranslator.h"

#include "DatasmithMVRImportOptions.h"
#include "Factories/DMXLibraryFromMVRFactory.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "DatasmithSceneFactory.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"
#include "Widgets/Notifications/SNotificationList.h"


DECLARE_LOG_CATEGORY_CLASS(LogDatasmithMVRNativeTranslator, Log, All);

bool FDatasmithMVRNativeTranslator::LoadScene(TSharedRef<IDatasmithScene> InOutScene)
{
	const bool bNativeTranslatorSuccess = FDatasmithNativeTranslator::LoadScene(InOutScene);
	if (!bNativeTranslatorSuccess)
	{
		return false;
	}

	TArray<TObjectPtr<UDatasmithOptionsBase>> ImportOptionsArray;
	GetSceneImportOptions(ImportOptionsArray);

	const TObjectPtr<UDatasmithOptionsBase>* ImportOptionsPtr = Algo::FindByPredicate(ImportOptionsArray, [](const UDatasmithOptionsBase* ImportOptions)
		{
			return ImportOptions->GetClass() == UDatasmithMVRImportOptions::StaticClass();
		});
	UDatasmithMVRImportOptions* MVRImportOptions = ImportOptionsPtr ? Cast<UDatasmithMVRImportOptions>(*ImportOptionsPtr) : nullptr;
	if (MVRImportOptions && !MVRImportOptions->bImportMVR)
	{
		// Importing MVR is turned off in options. Acknowledge the scene was successfully imported by the native translator.
		return true;
	}

	FString MVRFilePathAndName;
	if (!FindMVRFile(MVRFilePathAndName))
	{
		// If there is no occurence of MVR, acknowledge the scene was successfully imported by the native translator.
		return true;
	}

	// Interchange imports on its own thread
	if (!IsInGameThread())
	{
		//  MVR support via Interchange is prevented as it ingores any changes made to the scene here.
		AsyncTask(ENamedThreads::GameThread, []()
			{
				const FNotificationInfo Info(NSLOCTEXT("FDatasmithMVRNativeTranslator", "InterchangeNotSupportedNotification", "MVR Import is not supported when importing Datasmith via interchange"));
				FSlateNotificationManager::Get().AddNotification(Info);
			});

		// MVR is not supported. Acknowledge the scene was otherwise successfully import via Interchange.
		return true;
	}

	InOutScene->SetHost(TEXT("DatasmithMVRNativeTranslator"));
	InOutScene->SetProductName(TEXT("DatasmithMVRNativeTranslator"));

	UDMXLibrary* DMXLibrary = CreateDMXLibraryFromMVR(MVRFilePathAndName);
	if (DMXLibrary)
	{
		ReplaceMVRActorsWithMVRSceneActor(InOutScene, DMXLibrary);
	}
	else
	{
		const FNotificationInfo Info(NSLOCTEXT("FDatasmithMVRNativeTranslator", "FailedToImportMVRFileNotification", "Failed to import MVR. See log for details."));
		FSlateNotificationManager::Get().AddNotification(Info);
		// Acknowledge the scene was successfully imported by the native translator.
	}

	return true;
}

void FDatasmithMVRNativeTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	FDatasmithNativeTranslator::GetSceneImportOptions(Options);

	Options.Add(Datasmith::MakeOptionsPtr<UDatasmithMVRImportOptions>());
}

bool FDatasmithMVRNativeTranslator::FindMVRFile(FString& OutMVRFilePathAndName) const
{
	const FString DatasmithFilePathAndName = GetSource().GetSourceFile();
	const FString DatasmithFileName = FPaths::GetBaseFilename(DatasmithFilePathAndName);
	const FString DatasmithPath = FPaths::GetPath(DatasmithFilePathAndName);
	const FString DatasmithAssetsPath = FPaths::GetPath(DatasmithFilePathAndName) / DatasmithFileName + TEXT("_Assets");

	const FString MVRFilename = FPaths::GetBaseFilename(DatasmithFilePathAndName) + TEXT(".mvr");
	if (FPaths::FileExists(DatasmithPath / MVRFilename))
	{
		OutMVRFilePathAndName = DatasmithPath / MVRFilename;
		return true;
	}
	else if (FPaths::FileExists(DatasmithAssetsPath / MVRFilename))
	{
		OutMVRFilePathAndName = DatasmithAssetsPath / MVRFilename;
		return true;
	}

	// Vectorworks exports .udatasmith files with underscores, but MVR without underscores by default
	const FString MVRFilenameNoUnderscores = MVRFilename.Replace(TEXT("_"), TEXT(" "));
	if (FPaths::FileExists(DatasmithPath / MVRFilenameNoUnderscores))
	{
		OutMVRFilePathAndName = DatasmithPath / MVRFilenameNoUnderscores;
		return true;
	}
	else if (FPaths::FileExists(DatasmithAssetsPath / MVRFilenameNoUnderscores))
	{
		OutMVRFilePathAndName = DatasmithAssetsPath / MVRFilenameNoUnderscores;
		return true;
	}

	return false;
}

UDMXLibrary* FDatasmithMVRNativeTranslator::CreateDMXLibraryFromMVR(const FString& MVRFilePathAndName) const
{
	if (!ensureMsgf(FPaths::FileExists(MVRFilePathAndName), TEXT("Tried to create a DMX Library from MVR, but the MVR file doesn't exist anymore.")))
	{
		return nullptr;
	}

	UDMXLibraryFromMVRFactory* DMXLibraryFromMVRFactory = NewObject<UDMXLibraryFromMVRFactory>();

	const FString SceneName = GetSource().GetSceneName();
	const FString DMXLibraryName = FPaths::GetBaseFilename(TEXT("DMXLibrary_") + SceneName);
	const FString PackageName = TEXT("/Game") / SceneName / DMXLibraryName;
	UPackage* DMXLibraryAssetPackage = CreatePackage(*PackageName);
	DMXLibraryAssetPackage->FullyLoad();

	bool bOperationCanceled;
	UObject* NewDMXLibraryObject = DMXLibraryFromMVRFactory->FactoryCreateFile(UDMXLibrary::StaticClass(), DMXLibraryAssetPackage, *DMXLibraryName, RF_Public | RF_Standalone | RF_Transactional, MVRFilePathAndName, nullptr, GWarn, bOperationCanceled);
	if (bOperationCanceled)
	{
		return nullptr;
	}

	if (UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(NewDMXLibraryObject))
	{
		FAssetRegistryModule::AssetCreated(DMXLibrary);
		DMXLibrary->MarkPackageDirty();

		return DMXLibrary;
	}

	return nullptr;
}

void FDatasmithMVRNativeTranslator::ReplaceMVRActorsWithMVRSceneActor(const TSharedRef<IDatasmithScene>& InOutScene, UDMXLibrary* DMXLibrary) const
{
	if (!ensureMsgf(DMXLibrary, TEXT("Tried to create an MVR scene in the datasmith scene, but DMXLibrary is null.")))
	{
		return;
	}

	// Create the custom MVR Scene Actor element
	static const FString MVRSceneActorName = TEXT("MVR Scene Actor");
	static const FString MVRSceneActorPath = TEXT("/Script/DMXRuntime.DMXMVRSceneActor");

	const TSharedRef<IDatasmithCustomActorElement> CustomMVRSceneActorElement = FDatasmithSceneFactory::CreateCustomActor(*MVRSceneActorName);
	InOutScene->AddActor(CustomMVRSceneActorElement);
	CustomMVRSceneActorElement->SetClassOrPathName(*MVRSceneActorPath);
	CustomMVRSceneActorElement->SetIsAComponent(false);
	CustomMVRSceneActorElement->SetRotation(FQuat::Identity);
	CustomMVRSceneActorElement->SetScale(FVector::OneVector);
	CustomMVRSceneActorElement->SetTranslation(FVector::ZeroVector);

	// Set the DMX Library path as metadata of the custom actor element
	const TSoftObjectPtr<UDMXLibrary> SoftDMXLibraryObject = DMXLibrary;
	const TSharedRef<IDatasmithKeyValueProperty> FixturePatchKeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty(TEXT("DMXLibraryPath"));
	FixturePatchKeyValueProperty->SetValue(*SoftDMXLibraryObject.ToString());

	const TSharedRef<IDatasmithMetaDataElement> CustomActorMetaDataElement = FDatasmithSceneFactory::CreateMetaData(CustomMVRSceneActorElement->GetName());
	CustomActorMetaDataElement->AddProperty(FixturePatchKeyValueProperty);
	CustomActorMetaDataElement->SetAssociatedElement(CustomMVRSceneActorElement);

	InOutScene->AddMetaData(CustomActorMetaDataElement);

	// Remove native datasmith elements that correspond to an MVR Fixtures
	TArray<TSharedRef<IDatasmithMetaDataElement>> MetaDataElementsToRemove;
	TArray<TSharedRef<IDatasmithActorElement>> ActorElementsToRemove;

	const int32 NumMetaDatas = InOutScene->GetMetaDataCount();

	DMXLibrary->UpdateGeneralSceneDescription();
	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
	if (!ensureAlwaysMsgf(GeneralSceneDescription, TEXT("Trying to remove MVR scene elements from datasmith scene, but the DMX Library that was created for it does not provide a General Scene Description.")))
	{
		return;
	}

	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GeneralSceneDescription->GetFixtureNodes(FixtureNodes);
	for (const UDMXMVRFixtureNode* MVRFixture : FixtureNodes)
	{
		for (int32 MetaDataIndex = 0; MetaDataIndex < NumMetaDatas; MetaDataIndex++)
		{
			const TSharedPtr<IDatasmithMetaDataElement> MetaDataElement = InOutScene->GetMetaData(MetaDataIndex);

			// Try to find an MVR ID in MetaData. Note Vectorworks uses VW_UUID as that, so we search for that too.
			static const FString Generic_MVRUUIDName = TEXT("MVR_UUID");
			static const FString VW_MVRUUIDName = TEXT("VW_UUID");

			const TSharedPtr<IDatasmithKeyValueProperty> DatasmithKeyValueProperty = [&MetaDataElement]()
			{
				TSharedPtr<IDatasmithKeyValueProperty> Result = MetaDataElement->GetPropertyByName(*Generic_MVRUUIDName);
				if (!Result.IsValid())
				{
					Result = MetaDataElement->GetPropertyByName(*VW_MVRUUIDName);
				}

				return Result;
			}();

			if (!DatasmithKeyValueProperty.IsValid())
			{
				continue;
			}

			// Try to find the node that corresponds to the MVR node
			const FString MetaDataMVRUUIDString = FString(DatasmithKeyValueProperty->GetValue());
			FGuid MetaDataMVRUUID;
			if (!FGuid::Parse(MetaDataMVRUUIDString, MetaDataMVRUUID))
			{
				continue;
			}

			if (MetaDataMVRUUID == MVRFixture->UUID)
			{
				const TSharedPtr<IDatasmithElement> AssociatedElementToRemove = MetaDataElement->GetAssociatedElement();

				const bool bValidElementType = AssociatedElementToRemove.IsValid() || AssociatedElementToRemove->IsA(EDatasmithElementType::Actor);
				if (!ensureAlwaysMsgf(bValidElementType, TEXT("DatasmithMVRNativeTranslator unexpected Element type when trying to remove MVR element in favor of an MVR Scene Actor.")))
				{
					continue;
				}

				const TSharedRef<IDatasmithActorElement> ActorElementToRemove = StaticCastSharedRef<IDatasmithActorElement>(AssociatedElementToRemove.ToSharedRef());
				MetaDataElementsToRemove.Add(MetaDataElement.ToSharedRef());
				ActorElementsToRemove.Add(ActorElementToRemove);
			}
		}
	}

	for (const TSharedRef<IDatasmithMetaDataElement>& MetaDataElementToRemove : MetaDataElementsToRemove)
	{
		InOutScene->RemoveMetaData(MetaDataElementToRemove);
	}
	for (const TSharedRef<IDatasmithActorElement>& ActorElementToRemove : ActorElementsToRemove)
	{
		InOutScene->RemoveActor(ActorElementToRemove, EDatasmithActorRemovalRule::RemoveChildren);
	}
}
