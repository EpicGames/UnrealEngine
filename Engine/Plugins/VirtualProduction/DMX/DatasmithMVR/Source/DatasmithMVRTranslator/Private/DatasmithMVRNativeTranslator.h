// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DatasmithNativeTranslator.h"

class UDatasmithMVRImportOptions;

class UDMXLibrary;


/** An override of the datasmith native translator that considers MVR assets aside or in the *_Assets folder of a .udatasmith file */
class FDatasmithMVRNativeTranslator 
	: public FDatasmithNativeTranslator
{
public:
	//~ Begin DatasmithNativeTranslator interface
	virtual FName GetFName() const override { return "DatasmithMVRNativeTranslator"; };
	virtual bool LoadScene(TSharedRef<IDatasmithScene> InOutScene) override;
	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	//~ End DatasmithNativeTranslator interface

private:
	/** Tries to find an MVR file next to or inside the datasmith folder */
	bool FindMVRFile(FString& OutMVRFilePathAndName) const;

	/** Creates a DMX Library from an MVR */
	UDMXLibrary* CreateDMXLibraryFromMVR(const FString& MVRFilePathAndName) const;

	/** Replaces actors in the Datasmith Scene that correspond to the MVR with the MVR Scene Actor */
	void ReplaceMVRActorsWithMVRSceneActor(const TSharedRef<IDatasmithScene>& InOutScene, UDMXLibrary* DMXLibrary) const;
};
