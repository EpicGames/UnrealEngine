// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMVRTranslatorModule.h"
#include "DatasmithMVRImportOptions.h" // IWYU pragma: keep
#include "DatasmithMVRNativeTranslator.h"


#define LOCTEXT_NAMESPACE "DatasmithMVRTranslatorModule"

void FDatasmithMVRTranslatorModule::StartupModule()
{
	FModuleManager::Get().LoadModule(TEXT("DatasmithTranslator"));

	// Override the default native translator (the translator for .udatasmith files) and register the translator implemented in this module
	SetDatasmithMVRNativeTanslatorEnabled(true);
}

void FDatasmithMVRTranslatorModule::ShutdownModule()
{
	Datasmith::UnregisterTranslator<FDatasmithMVRNativeTranslator>();
}

void FDatasmithMVRTranslatorModule::SetDatasmithMVRNativeTanslatorEnabled(bool bEnabled)
{
	if (bEnabled)
	{
		Datasmith::UnregisterTranslator<FDatasmithNativeTranslator>();
		Datasmith::RegisterTranslator<FDatasmithMVRNativeTranslator>();
	}
	else
	{
		Datasmith::UnregisterTranslator<FDatasmithMVRNativeTranslator>();
		Datasmith::RegisterTranslator<FDatasmithNativeTranslator>();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDatasmithMVRTranslatorModule, DatasmithMVRTranslator)
