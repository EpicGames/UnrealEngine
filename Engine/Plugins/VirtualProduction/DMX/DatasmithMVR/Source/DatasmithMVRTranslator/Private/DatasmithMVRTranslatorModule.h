// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDatasmithMVRTranslatorModule.h"


class FDatasmithMVRTranslatorModule
	: public IDatasmithMVRTranslatorModule
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface interface

	//~ Begin IDatasmithMVRNativeTanslator inteface
	virtual void SetDatasmithMVRNativeTanslatorEnabled(bool bEnabled) override;
	//~ End IDatasmithMVRNativeTranslator interface
};
