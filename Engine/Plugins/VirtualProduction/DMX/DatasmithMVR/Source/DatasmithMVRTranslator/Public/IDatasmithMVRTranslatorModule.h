// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class DATASMITHMVRTRANSLATOR_API IDatasmithMVRTranslatorModule
	: public IModuleInterface
{
public:
	static IDatasmithMVRTranslatorModule& Get()
	{
		return FModuleManager::Get().GetModuleChecked<IDatasmithMVRTranslatorModule>("DatasmithMVRTranslator");
	}

	/** Enables or disables the native translator (translator for .udatasmith files) implemented in this module. When the module is started, the translator is enabled */
	virtual void SetDatasmithMVRNativeTanslatorEnabled(bool bEnabled) = 0;
};
